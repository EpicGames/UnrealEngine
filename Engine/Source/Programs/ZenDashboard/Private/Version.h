// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// See https://semver.org/

#define ZEN_DASHBOARD_VERSION_MAJOR 1
#define ZEN_DASHBOARD_VERSION_MINOR 0
#define ZEN_DASHBOARD_VERSION_PATCH 0

#define ZEN_DASHBOARD_VERSION_STRING "1.00"

#ifdef ZEN_DASHBOARD_VERSION_ID_STRING
#define ZEN_DASHBOARD_VERSION_STRING_EX "v" ZEN_DASHBOARD_VERSION_STRING "-" ZEN_DASHBOARD_VERSION_ID_STRING
#else
#define ZEN_DASHBOARD_VERSION_STRING_EX "v" ZEN_DASHBOARD_VERSION_STRING
#endif
