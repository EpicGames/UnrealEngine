// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZenDashboard.h"
#include "Experimental/ZenServerInterface.h"
#include "Features/IModularFeatures.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformProcess.h"
#include "Misc/CompilationResult.h"
#include "Misc/MessageDialog.h"
#include "Misc/Optional.h"
#include "Misc/Timespan.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "RequiredProgramMainCPPInclude.h"
#include "ZenServiceInstanceManager.h"
#include "SMessageDialog.h"
#include "StandaloneRenderer.h"
#include "SZenCacheStatistics.h"
#include "SZenCidStoreStatistics.h"
#include "SZenProjectStatistics.h"
#include "SZenServiceStatus.h"
#include "SZenWorkspaces.h"
#include "Tasks/Task.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/STextBlock.h"
#include "ZenDashboardStyle.h"

#if PLATFORM_WINDOWS
#include "Runtime/Launch/Resources/Windows/Resource.h"
#include "Windows/WindowsApplication.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#include <shellapi.h>
#include "Windows/HideWindowsPlatformTypes.h"
#elif PLATFORM_LINUX
#include "UnixCommonStartup.h"
#elif PLATFORM_MAC
#include "Mac/MacProgramDelegate.h"
#include "LaunchEngineLoop.h"
#else
#error "Unsupported platform!"
#endif

#define LOCTEXT_NAMESPACE "ZenDashboard"

// These macros are not properly defined by UBT in the case of an engine program with bTreatAsEngineModule=true
// So define them here as a workaround
#define IMPLEMENT_ENCRYPTION_KEY_REGISTRATION()
#define IMPLEMENT_SIGNING_KEY_REGISTRATION()
IMPLEMENT_APPLICATION(ZenDashboard, "ZenDashboard");

static void OnRequestExit()
{
	RequestEngineExit(TEXT("ZenDashboard Closed"));
}

static void HideOnCloseOverride(const TSharedRef<SWindow>& WindowBeingClosed)
{
	WindowBeingClosed->HideWindow();
}

// Controls whether we want the ZenDashboard app to behave as a systray app.  Currently only supported for windows.
#define UE_ZENDASHBOARD_SYSTRAY 0

#if UE_ZENDASHBOARD_SYSTRAY && PLATFORM_WINDOWS
#define WM_TRAYICON (WM_USER + 1000)

#define CMD_USER 400
#define CMD_SHOWDASHBOARD					(CMD_USER +   1)
#define CMD_STARTZENSERVER					(CMD_USER +   2)
#define CMD_STOPZENSERVER					(CMD_USER +   3)
#define CMD_RESTARTZENSERVER				(CMD_USER +   4)
#define CMD_EXITDASHBOARD					(CMD_USER +   5)
#endif

class FZenDashboardApp
#if UE_ZENDASHBOARD_SYSTRAY && PLATFORM_WINDOWS
	: IWindowsMessageHandler
#endif
{
	FCriticalSection CriticalSection;
	FSlateApplication& Slate;
	TSharedPtr<SWindow> Window;
	TSharedPtr<SNotificationItem> CompileNotification;
	TSharedPtr<UE::Zen::FServiceInstanceManager> ServiceInstanceManager;
	TArray<FSimpleDelegate> MainThreadTasks;
	TSharedPtr<SZenWorkspaces> ZenWorkspacesWidget;

	std::atomic<bool> bLatentExclusiveOperationActive = false;

#if UE_ZENDASHBOARD_SYSTRAY && PLATFORM_WINDOWS
	bool ProcessMessage(HWND hwnd, uint32 msg, WPARAM wParam, LPARAM lParam, int32& OutResult) override
	{
		switch (msg)
		{
		case WM_TRAYICON:
			if (LOWORD(lParam) == WM_LBUTTONDBLCLK)
			{
				Window->ShowWindow();
				return true;
			}
			else if (LOWORD(lParam) == WM_RBUTTONUP)
			{
				HMENU Menu = CreatePopupMenu();

				auto AddMenuItem = [Menu](const wchar_t* Text, uint32 Id)
				{
					MENUITEMINFOW MenuItemInfo = {
						sizeof(MenuItemInfo),
						MIIM_TYPE|MIIM_STATE|MIIM_ID,
						MFT_STRING,
						MFS_ENABLED,
						Id,
					};
					MenuItemInfo.dwTypeData = LPWSTR(Text);
					InsertMenuItemW(Menu, 999, true, &MenuItemInfo);
				};

				AddMenuItem(*LOCTEXT("Show_ZenDashboard", "Zen Dashboard").ToString(), CMD_SHOWDASHBOARD);
				AddMenuItem(*LOCTEXT("Start_ZenServer", "Start Zen Server").ToString(), CMD_STARTZENSERVER);
				AddMenuItem(*LOCTEXT("Stop_ZenServer", "Stop Zen Server").ToString(), CMD_STOPZENSERVER);
				AddMenuItem(*LOCTEXT("Restart_ZenServer", "Restart Zen Server").ToString(), CMD_RESTARTZENSERVER);
				AddMenuItem(*LOCTEXT("Exit", "Exit").ToString(), CMD_EXITDASHBOARD);

				POINT MousePos;
				GetCursorPos(&MousePos);
				BOOL Ret = TrackPopupMenu(
					Menu, TPM_RIGHTALIGN|TPM_BOTTOMALIGN,
					MousePos.x, MousePos.y,
					0, hwnd, nullptr);

				DestroyMenu(Menu);
			}
			break;
		case WM_COMMAND:
			switch (LOWORD(wParam))
			{
			case CMD_SHOWDASHBOARD:
				Window->ShowWindow();
				break;
			case CMD_STARTZENSERVER:
				if (CanExecuteExclusiveAction())
				{
					StartZenServer();
				}
				break;
			case CMD_STOPZENSERVER:
				if (CanExecuteExclusiveAction())
				{
					StopZenServer();
				}
				break;
			case CMD_RESTARTZENSERVER:
				if (CanExecuteExclusiveAction())
				{
					RestartZenServer();
				}
				break;
			case CMD_EXITDASHBOARD:
				if (CanExecuteExclusiveAction())
				{
					ExitDashboard();
				}
				break;
			}
			break;
		}

		return false;
	}
#endif // UE_ZENDASHBOARD_SYSTRAY && PLATFORM_WINDOWS

	void ExitDashboard()
	{
		FSlateApplication::Get().RequestDestroyWindow(Window.ToSharedRef());
	}

	bool CanExecuteExclusiveAction()
	{
		return !bLatentExclusiveOperationActive;
	}

	void StartZenServer()
	{
		UE::Zen::FZenLocalServiceRunContext RunContext;
		if (UE::Zen::TryGetLocalServiceRunContext(RunContext))
		{
			UE::Zen::StartLocalService(RunContext);
		}
	}

	void StopZenServer()
	{
		UE::Zen::FZenLocalServiceRunContext RunContext;
		if (UE::Zen::TryGetLocalServiceRunContext(RunContext))
		{
			UE::Zen::StopLocalService(*RunContext.GetDataPath());
		}
	}

	void RestartZenServer()
	{
		StopZenServer();
		StartZenServer();
	}

	void DeleteDataAndRestartZenServer()
	{
		UE::Zen::FZenLocalServiceRunContext RunContext;
		if (UE::Zen::TryGetLocalServiceRunContext(RunContext))
		{
			if (UE::Zen::IsLocalServiceRunning(*RunContext.GetDataPath()))
			{
				FText Title = LOCTEXT("ConfirmToDeleteZenServerData_Title", "Zen Server Data Delete");
				FText ConfirmMessage = FText::Format(LOCTEXT("ConfirmToDeleteZenServerData", "You are about to delete all local Zen Server data at:\n\n{0}\n\nThis will require you to build or download all data stored there which can take a long time. Are you sure you want to proceed?"),
					FText::FromString(RunContext.GetDataPath()));
				EAppReturnType::Type OkToDelete = OpenModalMessageDialog_Internal(EAppMsgCategory::Warning, EAppMsgType::YesNo, ConfirmMessage, Title, Window);
				if (OkToDelete == EAppReturnType::No)
				{
					return;
				}

				StopZenServer();

				bLatentExclusiveOperationActive = true;

				UE::Tasks::Launch(TEXT("DeleteZenDataAndRestart"),
					[this, DataPath = RunContext.GetDataPath()] ()
					{
						FPlatformProcess::Sleep(10.0f);
						FPlatformFileManager::Get().GetPlatformFile().DeleteDirectoryRecursively(*DataPath);

						StartZenServer();

						bLatentExclusiveOperationActive = false;
					},
					UE::Tasks::ETaskPriority::BackgroundNormal
				);
			}
		}
	}

	void GatherDiagnostics()
	{
	}

	void LaunchUtilityCommandPrompt()
	{
#if PLATFORM_WINDOWS
		TStringBuilder<128> Args;
		FString UtilityPath = UE::Zen::GetLocalInstallUtilityPath();
		Args << TEXT("/k \"") << UtilityPath << TEXT("\" --help");
		FProcHandle UtilProcHandle = FPlatformProcess::CreateProc(TEXT("cmd.exe"), Args.ToString(), false, false, false, nullptr, 0, *FPaths::GetPath(UtilityPath), nullptr);
#else
		// TODO: Need to get the equivalent for linux/mac
#endif
	}

	void RunGC()
	{
		if (TSharedPtr<UE::Zen::FZenServiceInstance> ZenServiceInstance = ServiceInstanceManager->GetZenServiceInstance())
		{
			ZenServiceInstance->RequestGC();
		}
	}

	void RunGCOneWeek()
	{
		if (TSharedPtr<UE::Zen::FZenServiceInstance> ZenServiceInstance = ServiceInstanceManager->GetZenServiceInstance())
		{
			uint32 CacheDuration = static_cast<uint32>(FTimespan::FromDays(7).GetTotalSeconds());
			ZenServiceInstance->RequestGC(nullptr, &CacheDuration);
		}
	}

	void RunGCOneDay()
	{
		if (TSharedPtr<UE::Zen::FZenServiceInstance> ZenServiceInstance = ServiceInstanceManager->GetZenServiceInstance())
		{
			uint32 CacheDuration = static_cast<uint32>(FTimespan::FromDays(1).GetTotalSeconds());
			ZenServiceInstance->RequestGC(nullptr, &CacheDuration);
		}
	}

	TSharedRef< SWidget > MakeMainMenu()
	{
		FMenuBarBuilder MenuBuilder( nullptr );
		{
			// Control
			MenuBuilder.AddPullDownMenu(
				LOCTEXT( "FileMenu", "File" ),
				LOCTEXT( "FileMenu_ToolTip", "Opens the file menu" ),
				FOnGetContent::CreateRaw( this, &FZenDashboardApp::FillFileMenu ) );

			// Control
			MenuBuilder.AddPullDownMenu(
				LOCTEXT( "ToolsMenu", "Tools" ),
				LOCTEXT( "ToolsMenu_ToolTip", "Opens the tools menu" ),
				FOnGetContent::CreateRaw( this, &FZenDashboardApp::FillToolsMenu ) );
		}

		// Create the menu bar
		TSharedRef< SWidget > MenuBarWidget = MenuBuilder.MakeWidget();
		MenuBarWidget->SetVisibility( EVisibility::Visible ); // Work around for menu bar not showing on Mac

		return MenuBarWidget;
	}

	TSharedRef<SWidget> FillFileMenu()
	{
		const bool bCloseSelfOnly = false;
		const bool bSearchable = false;
		const bool bRecursivelySearchable = false;

		FMenuBuilder MenuBuilder(true,
			nullptr,
			TSharedPtr<FExtender>(),
			bCloseSelfOnly,
			&FCoreStyle::Get(),
			bSearchable,
			NAME_None,
			bRecursivelySearchable);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("Exit", "Exit"),
			LOCTEXT("Exit_ToolTip", "Exits the Zen Dashboard"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw( this, &FZenDashboardApp::ExitDashboard ),
				FCanExecuteAction::CreateRaw(this, &FZenDashboardApp::CanExecuteExclusiveAction)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		return MenuBuilder.MakeWidget();
	}

	TSharedRef<SWidget> FillToolsMenu()
	{
		const bool bCloseSelfOnly = false;
		const bool bSearchable = false;
		const bool bRecursivelySearchable = false;

		FMenuBuilder MenuBuilder(true,
			nullptr,
			TSharedPtr<FExtender>(),
			bCloseSelfOnly,
			&FCoreStyle::Get(),
			bSearchable,
			NAME_None,
			bRecursivelySearchable);

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("ServiceControls", "Service controls"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("Start_ZenServer", "Start Zen Server"),
				LOCTEXT("Start_ZenServer_ToolTip", "Ensures ZenServer is started"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FZenDashboardApp::StartZenServer),
					FCanExecuteAction::CreateRaw(this, &FZenDashboardApp::CanExecuteExclusiveAction)
				),
				NAME_None,
				EUserInterfaceActionType::Button
			);
			MenuBuilder.AddMenuEntry(
				LOCTEXT("Stop_ZenServer", "Stop Zen Server"),
				LOCTEXT("Stop_ZenServer_ToolTip", "Ensures ZenServer is stopped"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FZenDashboardApp::StopZenServer),
					FCanExecuteAction::CreateRaw(this, &FZenDashboardApp::CanExecuteExclusiveAction)
				),
				NAME_None,
				EUserInterfaceActionType::Button
			);
			MenuBuilder.AddMenuEntry(
				LOCTEXT("Restart_ZenServer", "Restart Zen Server"),
				LOCTEXT("Restart_ZenServer_ToolTip", "Stops ZenServer if it is running then ensures it is running"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FZenDashboardApp::RestartZenServer),
					FCanExecuteAction::CreateRaw(this, &FZenDashboardApp::CanExecuteExclusiveAction)
				),
				NAME_None,
				EUserInterfaceActionType::Button
			);

			MenuBuilder.AddWrapperSubMenu(LOCTEXT("Advanced", "Advanced"), LOCTEXT("Advanced_ToolTip", "Advanced service control commands"),
				FOnGetContent::CreateLambda([this]() -> TSharedRef<SWidget>
					{
						const bool bCloseSelfOnly = false;
						const bool bSearchable = false;
						const bool bRecursivelySearchable = false;

						FMenuBuilder SubMenuBuilder(true,
							nullptr,
							TSharedPtr<FExtender>(),
							bCloseSelfOnly,
							&FCoreStyle::Get(),
							bSearchable,
							NAME_None,
							bRecursivelySearchable);

						SubMenuBuilder.AddMenuEntry(
							LOCTEXT("DeleteDataAndRestart_ZenServer", "Delete data and restart Zen Server"),
							LOCTEXT("DeleteDataAndRestart_ZenServer_ToolTip", "Stops ZenServer if it is running, deletes ALL of its data, then starts it from a blank state"),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateRaw(this, &FZenDashboardApp::DeleteDataAndRestartZenServer),
								FCanExecuteAction::CreateRaw(this, &FZenDashboardApp::CanExecuteExclusiveAction)
							),
							NAME_None,
							EUserInterfaceActionType::Button
						);
						SubMenuBuilder.AddMenuEntry(
							LOCTEXT("ResetStats_ZenServer", "Reset stat counters"),
							LOCTEXT("ResetStats_ZenServer_ToolTip", "Reset the cache stats"),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateRaw(this, &FZenDashboardApp::DeleteDataAndRestartZenServer),
								FCanExecuteAction::CreateRaw(this, &FZenDashboardApp::CanExecuteExclusiveAction)
							),
							NAME_None,
							EUserInterfaceActionType::Button
						);

						return SubMenuBuilder.MakeWidget();
					})
				, FSlateIcon()
			);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("Debug", "Debug"));
		{
			// TODO: The "Gather diagnostic info" feature is off for now and needs to be filled in as part of a subsequent change
			// MenuBuilder.AddMenuEntry(
			// 	LOCTEXT("GatherDiagnosticInfo", "Gather diagnostic info"),
			// 	LOCTEXT("GatherDiagnosticInfo_ToolTip", "Gathers information for diagnosing issues"),
			// 	FSlateIcon(),
			// 	FUIAction(
			// 		FExecuteAction::CreateRaw(this, &FZenDashboardApp::GatherDiagnostics),
			// 		FCanExecuteAction()
			// 	),
			// 	NAME_None,
			// 	EUserInterfaceActionType::Button
			// );
			MenuBuilder.AddMenuEntry(
				LOCTEXT("LaunchUtilityCommandPrompt", "Launch utility command prompt"),
				LOCTEXT("LaunchUtilityCommandPrompt_ToolTip", "Launches command prompt for issuing zen server specific commands"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FZenDashboardApp::LaunchUtilityCommandPrompt),
					FCanExecuteAction()
				),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("Workspace", "Workspace"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("LaunchUtilityCommandPrompt", "Manage workspaces"),
				LOCTEXT("LaunchUtilityCommandPrompt_ToolTip", "Manage Zen workspaces: list, create, delete"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FZenDashboardApp::LaunchUtilityCommandPrompt),
					FCanExecuteAction()
				),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("Disk", "Disk"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("FreeUnusedDiskSpace", "Free unused disk space"),
				LOCTEXT("FreeUnusedDiskSpace_ToolTip", "Frees unused disk space by running a garbage collection process"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FZenDashboardApp::RunGC),
					FCanExecuteAction::CreateRaw(this, &FZenDashboardApp::CanExecuteExclusiveAction)
				),
				NAME_None,
				EUserInterfaceActionType::Button
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("FreeUnusedDiskSpaceOneWeek", "Free unused disk space (1 week age limit)"),
				LOCTEXT("FreeUnusedDiskSpaceOneWeek_ToolTip", "Frees unused disk space by running a garbage collection process, only keeps data that was used in the past week"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FZenDashboardApp::RunGCOneWeek),
					FCanExecuteAction::CreateRaw(this, &FZenDashboardApp::CanExecuteExclusiveAction)
				),
				NAME_None,
				EUserInterfaceActionType::Button
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("FreeUnusedDiskSpaceOneDay", "Free unused disk space (1 day age limit)"),
				LOCTEXT("FreeUnusedDiskSpaceOneDay_ToolTip", "Frees unused disk space by running a garbage collection process, only keeps data that was used in the past day"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FZenDashboardApp::RunGCOneDay),
					FCanExecuteAction::CreateRaw(this, &FZenDashboardApp::CanExecuteExclusiveAction)
				),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

public:
	FZenDashboardApp(FSlateApplication& InSlate)
		: Slate(InSlate)
	{
		ServiceInstanceManager = MakeShared<UE::Zen::FServiceInstanceManager>();
		InstallMessageDialogOverride();
	}

	virtual ~FZenDashboardApp()
	{
		RemoveMessageDialogOverride();
	}

	void Run()
	{
		const bool bSystemTrayMode = !!UE_ZENDASHBOARD_SYSTRAY && !!PLATFORM_WINDOWS;
		const bool bShowWindow = !(FParse::Param(FCommandLine::Get(), TEXT("Minimized")) && bSystemTrayMode);
		
		Window =
			SNew(SWindow)
			.Title(GetWindowTitle())
			.ClientSize(FVector2D(384.0f, 128.0f))
			.ActivationPolicy(EWindowActivationPolicy::Always)
			.SizingRule(ESizingRule::Autosized)
			.IsTopmostWindow(false)
			.FocusWhenFirstShown(false)
			.SupportsMaximize(false)
			.SupportsMinimize(true)
			.HasCloseButton(true);

		TSharedPtr<SWidget> WindowContent = CreateWindowContent();
		Window->SetContent(WindowContent.ToSharedRef());

		if (bSystemTrayMode)
		{
			Window->SetRequestDestroyWindowOverride(FRequestDestroyWindowOverride::CreateStatic(&HideOnCloseOverride));
		}

		Slate.AddWindow(Window.ToSharedRef(), bShowWindow);

		// Setting focus seems to have to happen after the Window has been added
		Slate.ClearKeyboardFocus(EFocusCause::Cleared);

		// 
#if UE_ZENDASHBOARD_SYSTRAY && PLATFORM_WINDOWS
		if (bSystemTrayMode)
		{
			((FWindowsApplication*)Slate.Get().GetPlatformApplication().Get())->AddMessageHandler(*this);

			NOTIFYICONDATAW NotifyIconData;
			memset(&NotifyIconData, 0, sizeof(NotifyIconData));
			NotifyIconData.cbSize = sizeof(NotifyIconData);
			NotifyIconData.hWnd = (HWND)Window->GetNativeWindow()->GetOSWindowHandle();
			NotifyIconData.uID = 0;
			NotifyIconData.uCallbackMessage = WM_USER + 1000;
			NotifyIconData.uFlags = NIF_ICON | NIF_MESSAGE;
			NotifyIconData.hIcon = ::LoadIconW(GetModuleHandleW(nullptr), MAKEINTRESOURCE(IDICON_UEGame));
			Shell_NotifyIcon(NIM_ADD, &NotifyIconData);
		}
#endif
		double DeltaTime = 0.0;
		double LastTime = FPlatformTime::Seconds();
		const float IdealFrameTime = 1.0f / 60;
		const float BackgroundFrameTime = 1.0f / 4;

		// loop until the app is ready to quit
		while (!IsEngineExitRequested())
		{
			BeginExitIfRequested();

			FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
			FTSTicker::GetCoreTicker().Tick(DeltaTime);
			Slate.PumpMessages();
			Slate.Tick();

			const float FrameTime = !FPlatformApplicationMisc::IsThisApplicationForeground() && (FPlatformTime::Seconds() - FSlateApplication::Get().GetLastUserInteractionTime()) > 5 ? BackgroundFrameTime : IdealFrameTime;
			FPlatformProcess::Sleep(FMath::Max<float>(0.f, FrameTime - static_cast<float>(FPlatformTime::Seconds() - LastTime)));

			DeltaTime = FPlatformTime::Seconds() - LastTime;
			LastTime = FPlatformTime::Seconds();

			UE::Stats::FStats::AdvanceFrame(false);
			FCoreDelegates::OnEndFrame.Broadcast();

			GFrameCounter++;
		}

#if UE_ZENDASHBOARD_SYSTRAY && PLATFORM_WINDOWS
		if (bSystemTrayMode)
		{
			memset(&NotifyIconData, 0, sizeof(NotifyIconData));
			NotifyIconData.cbSize = sizeof(NotifyIconData);
			NotifyIconData.uID = 0;
			NotifyIconData.hWnd = (HWND)Window->GetNativeWindow()->GetOSWindowHandle();
			Shell_NotifyIcon(NIM_DELETE, &NotifyIconData);
		}
#endif

		// Make sure the window is hidden, because it might take a while for the background thread to finish.
		Window->HideWindow();
	}

private:
	FText GetWindowTitle()
	{
		return LOCTEXT("WindowTitle", "Unreal Zen Dashboard");
	}

	EAppReturnType::Type OnModalMessageDialog(EAppMsgCategory InMessageCategory, EAppMsgType::Type InMessage, const FText& InText, const FText& InTitle)
	{
		if (IsInGameThread() && FSlateApplication::IsInitialized() && FSlateApplication::Get().CanAddModalWindow())
		{
			return OpenModalMessageDialog_Internal(InMessageCategory, InMessage, InText, InTitle, Window);
		}
		else
		{
			return FPlatformMisc::MessageBoxExt(InMessage, *InText.ToString(), *InTitle.ToString());
		}
	}

	void InstallMessageDialogOverride()
	{
		FCoreDelegates::ModalMessageDialog.BindRaw(this, &FZenDashboardApp::OnModalMessageDialog);
	}

	void RemoveMessageDialogOverride()
	{
		FCoreDelegates::ModalMessageDialog.Unbind();
	}

	TSharedPtr<SWidget> CreateWindowContent()
	{
		TSharedPtr<SWidget> WindowContent = 
			SNew(SBorder)
				.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)

					// Menu
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						MakeMainMenu()
					]

					// Status panel
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 10.0f, 0.0f, 0.0f)
					.VAlign(VAlign_Top)
					[
						SNew(SZenServiceStatus)
						.ZenServiceInstance(ServiceInstanceManager.ToSharedRef(), &UE::Zen::FServiceInstanceManager::GetZenServiceInstance)
					]

					// Stats panel
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 10.0f, 0.0f, 0.0f)
					.VAlign(VAlign_Top)
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Top)
						[
							SNew(SZenCacheStatistics)
							.ZenServiceInstance(ServiceInstanceManager.ToSharedRef(), &UE::Zen::FServiceInstanceManager::GetZenServiceInstance)
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Top)
						[
							SNew(SZenProjectStatistics)
							.ZenServiceInstance(ServiceInstanceManager.ToSharedRef(), &UE::Zen::FServiceInstanceManager::GetZenServiceInstance)
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Top)
						[
							SNew(SZenCidStoreStatistics)
							.ZenServiceInstance(ServiceInstanceManager.ToSharedRef(), &UE::Zen::FServiceInstanceManager::GetZenServiceInstance)
						]
					]

					// Workspaces panel
					+ SVerticalBox::Slot()
					.MaxHeight(300)
					.Padding(4.f, 10.f, 4.f, 0.f)
					.FillHeight(1.0f)
					.VAlign(VAlign_Top)
					[
						SAssignNew(ZenWorkspacesWidget, SZenWorkspaces)
						.ZenServiceInstance(ServiceInstanceManager.ToSharedRef(), &UE::Zen::FServiceInstanceManager::GetZenServiceInstance)
						.UpdateFrequency(5.f)
						.Visibility_Lambda([this]() -> EVisibility
						{
							static double LastTime = FPlatformTime::Seconds();
							const double CurrentTime = FPlatformTime::Seconds();
							const double DeltaTime = CurrentTime - LastTime;
							LastTime = CurrentTime;

							bool bShow = false;
							if (ServiceInstanceManager->GetZenServiceInstance().IsValid()
								&& ServiceInstanceManager->GetZenServiceInstance()->IsServiceRunning())
							{
								ZenWorkspacesWidget->UpdateWorkspaces(DeltaTime);
								bShow = ZenWorkspacesWidget->GetWorkspaceCount() > 0;
							}
							
							return bShow ? EVisibility::Visible : EVisibility::Collapsed;
						})
					]
				];

		return WindowContent;
	}
};

int ZenDashboardMain(const TCHAR* CmdLine)
{
	ON_SCOPE_EXIT
	{
		RequestEngineExit(TEXT("Exiting"));
		FEngineLoop::AppPreExit();
		FModuleManager::Get().UnloadModulesAtShutdown();
		FEngineLoop::AppExit();
	};
	
	const FTaskTagScope Scope(ETaskTag::EGameThread);

	// start up the main loop
	GEngineLoop.PreInit(CmdLine);

	FSystemWideCriticalSection SystemWideZenDashboardCritSec(TEXT("ZenDashboard"));
	if (!SystemWideZenDashboardCritSec.IsValid())
	{
		return true;
	}
	check(GConfig && GConfig->IsReadyForUse());

	// Initialize high DPI mode
	FSlateApplication::InitHighDPI(true);

	{
		// Create the platform slate application (what FSlateApplication::Get() returns)
		TSharedRef<FSlateApplication> Slate = FSlateApplication::Create(MakeShareable(FPlatformApplicationMisc::CreateApplication()));

		{
			// Initialize renderer
			TSharedRef<FSlateRenderer> SlateRenderer = GetStandardStandaloneRenderer();

			// Try to initialize the renderer. It's possible that we launched when the driver crashed so try a few times before giving up.
			bool bRendererInitialized = Slate->InitializeRenderer(SlateRenderer, true);
			if (!bRendererInitialized)
			{
				// Close down the Slate application
				FSlateApplication::Shutdown();
				return false;
			}

			// Set the normal UE IsEngineExitRequested() when outer frame is closed
			Slate->SetExitRequestedHandler(FSimpleDelegate::CreateStatic(&OnRequestExit));

			// Prepare the custom Slate styles
			FZenDashboardStyle::Initialize();

			// Set the icon
			FAppStyle::SetAppStyleSet(FZenDashboardStyle::Get());

			// Run the inner application loop
			FZenDashboardApp App(Slate.Get());
			App.Run();

			// Clean up the custom styles
			FZenDashboardStyle::Shutdown();
		}

		// Close down the Slate application
		FSlateApplication::Shutdown();
	}

	return true;
}

#if PLATFORM_WINDOWS
int WINAPI WinMain(_In_ HINSTANCE hCurrInstance, _In_opt_ HINSTANCE hPrevInstance, _In_ LPSTR lpCmdLine, _In_ int nShowCmd)
{
	hInstance = hCurrInstance;
	return ZenDashboardMain(GetCommandLineW())? 0 : 1;
}

#elif PLATFORM_LINUX
int main(int argc, char* argv[])
{
	return CommonUnixMain(argc, argv, &ZenDashboardMain);
}
#elif PLATFORM_MAC
int main(int argc, char* argv[])
{
	[MacProgramDelegate mainWithArgc : argc argv : argv programMain : ZenDashboardMain programExit : FEngineLoop::AppExit] ;
}
#else
#error "Unsupported platform!"
#endif

#undef LOCTEXT_NAMESPACE
