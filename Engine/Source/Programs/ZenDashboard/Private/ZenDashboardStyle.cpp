// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZenDashboardStyle.h"
#include "Misc/AssertionMacros.h"
#include "Containers/UnrealString.h"
#include "Misc/Paths.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"

#define EDITOR_IMAGE_BRUSH(RelativePath, ...) IMAGE_BRUSH("../Editor/Slate/" RelativePath, __VA_ARGS__)
#define EDITOR_IMAGE_BRUSH_SVG(RelativePath, ...) IMAGE_BRUSH_SVG("../Editor/Slate/" RelativePath, __VA_ARGS__)
#define ENGINE_IMAGE_BRUSH_SVG(RelativePath, ...) IMAGE_BRUSH_SVG("../Slate/" RelativePath, __VA_ARGS__)
#define EDITOR_BOX_BRUSH(RelativePath, ...) BOX_BRUSH("../Editor/Slate/" RelativePath, __VA_ARGS__)
#define EDITOR_BORDER_BRUSH(RelativePath, ...) BORDER_BRUSH("../Editor/Slate/" RelativePath, __VA_ARGS__)
#define RootToContentDir ContentFromEngine
#define RootToCoreContentDir ContentFromEngine

TSharedPtr< FSlateStyleSet > FZenDashboardStyle::StyleSet = nullptr;

void FZenDashboardStyle::Initialize()
{
	if(!StyleSet.IsValid())
	{
		StyleSet = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleSet);
	}
}

void FZenDashboardStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet);
	ensure(StyleSet.IsUnique());
	StyleSet.Reset();
}

namespace
{
	FString ContentFromEngine(const FString& RelativePath, const TCHAR* Extension)
	{
		static const FString ContentDir = FPaths::EngineDir() / TEXT("Content/Slate");
		return ContentDir / RelativePath + Extension;
	}
}

TSharedRef< FSlateStyleSet > FZenDashboardStyle::Create()
{
	TSharedRef<FSlateStyleSet> StyleRef = MakeShareable(new FSlateStyleSet("ZenDashboardStyle"));
	FSlateStyleSet& Style = StyleRef.Get();

	Style.SetParentStyleName("CoreStyle");

	const ISlateStyle* ParentStyle = FSlateStyleRegistry::FindSlateStyle("CoreStyle");
	const FTextBlockStyle NormalText = ParentStyle->GetWidgetStyle<FTextBlockStyle>("NormalText");

	const FVector2D Icon10x10(10.f, 10.f);
	const FVector2D Icon12x12(12.0f, 12.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);

	Style.Set( "AppIcon", new EDITOR_IMAGE_BRUSH_SVG( "Starship/Zen/Zen_24", Icon20x20) );

	Style.Set("Zen.WebBrowser", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/WebBrowser", Icon12x12));
	Style.Set("Zen.FolderExplore", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/ContentBrowser", Icon12x12));
	Style.Set("Zen.Clipboard", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Clipboard", Icon16x16));

	Style.Set("Icons.ChevronRight", new CORE_IMAGE_BRUSH_SVG("Starship/Common/chevron-right", Icon16x16));
	Style.Set("Icons.ChevronDown", new CORE_IMAGE_BRUSH_SVG("Starship/Common/chevron-down", Icon16x16));

	Style.Set("GenericCommands.Delete", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Delete", Icon12x12));
	Style.Set("Icons.Delete-small", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Delete", Icon10x10));
	Style.Set("Icons.Cross", new CORE_IMAGE_BRUSH_SVG("Starship/Common/close-small", Icon16x16));
	Style.Set("Icons.XCircle", new CORE_IMAGE_BRUSH_SVG("Starship/Common/close-circle", Icon12x12));
	Style.Set("Icons.ArrowLeft", new CORE_IMAGE_BRUSH_SVG("Starship/Common/arrow-left", Icon16x16));
	Style.Set("Icons.FolderOpen", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-open", Icon16x16));
	Style.Set("Icons.Check", new CORE_IMAGE_BRUSH_SVG("Starship/Common/check", Icon16x16));

	Style.Set("Icons.Add", new CORE_IMAGE_BRUSH_SVG("Starship/Common/plus", Icon16x16));

	const FTextBlockStyle DefaultText = FTextBlockStyle()
		.SetFont(DEFAULT_FONT("Bold", 10))
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetShadowOffset(FVector2D::ZeroVector)
		.SetShadowColorAndOpacity(FLinearColor::Black);

	// Set the client app styles
	Style.Set(TEXT("Code"), FTextBlockStyle(DefaultText)
		.SetFont(DEFAULT_FONT("Regular", 8))
		.SetColorAndOpacity(FSlateColor(FLinearColor::White * 0.8f))
	);

	Style.Set(TEXT("Title"), FTextBlockStyle(DefaultText)
		.SetFont(DEFAULT_FONT("Bold", 12))
	);

	Style.Set(TEXT("Status"), FTextBlockStyle(DefaultText)
		.SetColorAndOpacity(FSlateColor::UseSubduedForeground())
	);

	{
		// Navigation defaults
		const FLinearColor NavHyperlinkColor(0.03847f, 0.33446f, 1.0f);
		const FTextBlockStyle NavigationHyperlinkText = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 10))
			.SetColorAndOpacity(NavHyperlinkColor);

		const FButtonStyle NavigationHyperlinkButton = FButtonStyle()
			.SetNormal(BORDER_BRUSH("Old/HyperlinkDotted", FMargin(0, 0, 0, 3 / 16.0f), NavHyperlinkColor))
			.SetPressed(FSlateNoResource())
			.SetHovered(BORDER_BRUSH("Old/HyperlinkUnderline", FMargin(0, 0, 0, 3 / 16.0f), NavHyperlinkColor));

		FHyperlinkStyle NavigationHyperlink = FHyperlinkStyle()
			.SetUnderlineStyle(NavigationHyperlinkButton)
			.SetTextStyle(NavigationHyperlinkText)
			.SetPadding(FMargin(0.0f));

		Style.Set("NavigationHyperlink", NavigationHyperlink);
	}

	FSlateBrush* GenericWhiteBox = new IMAGE_BRUSH( "Old/White", Icon16x16 );

	// Scrollbar
	const FScrollBarStyle ScrollBar = FScrollBarStyle()
		.SetVerticalTopSlotImage(IMAGE_BRUSH("Common/Scrollbar_Background_Vertical", FVector2D(8, 8)))
		.SetVerticalBottomSlotImage(IMAGE_BRUSH("Common/Scrollbar_Background_Vertical", FVector2D(8, 8)))
		.SetHorizontalTopSlotImage(IMAGE_BRUSH("Common/Scrollbar_Background_Horizontal", FVector2D(8, 8)))
		.SetHorizontalBottomSlotImage(IMAGE_BRUSH("Common/Scrollbar_Background_Horizontal", FVector2D(8, 8)))
		.SetNormalThumbImage(BOX_BRUSH("Common/Scrollbar_Thumb", FMargin(4.f / 16.f)))
		.SetDraggedThumbImage(BOX_BRUSH("Common/Scrollbar_Thumb", FMargin(4.f / 16.f)))
		.SetHoveredThumbImage(BOX_BRUSH("Common/Scrollbar_Thumb", FMargin(4.f / 16.f)));

	Style.Set("Log.TextBox", FEditableTextBoxStyle(FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
		.SetBackgroundImageNormal( BOX_BRUSH("Common/WhiteGroupBorder", FMargin(4.0f/16.0f)))
		.SetBackgroundImageHovered( BOX_BRUSH("Common/WhiteGroupBorder", FMargin(4.0f/16.0f)))
		.SetBackgroundImageFocused(BOX_BRUSH("Common/WhiteGroupBorder", FMargin(4.0f / 16.0f)))
		.SetBackgroundImageReadOnly(BOX_BRUSH("Common/WhiteGroupBorder", FMargin(4.0f / 16.0f)))
		.SetBackgroundColor( FLinearColor(0.015f, 0.015f, 0.015f) )
		.SetScrollBarStyle(ScrollBar)
		);

	return StyleRef;
}

const ISlateStyle& FZenDashboardStyle::Get()
{
	return *StyleSet;
}

#undef EDITOR_IMAGE_BRUSH
#undef EDITOR_IMAGE_BRUSH_SVG
#undef EDITOR_BOX_BRUSH
#undef EDITOR_BORDER_BRUSH
#undef RootToContentDir
