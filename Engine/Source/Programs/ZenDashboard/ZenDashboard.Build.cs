// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ZenDashboard : ModuleRules
{
	public ZenDashboard(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		bTreatAsEngineModule = true;
		PrivateDefinitions.Add("WITH_CURL=1");

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"Core",
				"HTTP",
				"Json",
				"Projects",
				"Slate",
				"SlateCore",
				"SSL",
				"StandaloneRenderer",
				"StorageServerWidgets",
				"Zen"
			});

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"UnixCommonStartup"
				}
			);
		}

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) && Target.WindowsPlatform.bUseXCurl)
		{
			PublicDependencyModuleNames.Add("XCurl");
		}
		else
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "libcurl");
		}

		AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenSSL");
		PublicIncludePathModuleNames.Add("Launch");
	}
}
