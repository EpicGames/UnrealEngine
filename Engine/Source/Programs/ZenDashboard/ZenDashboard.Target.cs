// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class ZenDashboardTarget : TargetRules
{
	public ZenDashboardTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LaunchModuleName = "ZenDashboard";

		bUseXGEController				= false;
		bLoggingToMemoryEnabled			= true;
		bUseLoggingInShipping			= true;
		bCompileWithAccessibilitySupport= false;
		bWithServerCode					= false;
		bCompileNavmeshClusterLinks		= false;
		bCompileNavmeshSegmentLinks		= false;
		bCompileRecast					= false;
		bCompileICU 					= false;
		bWithLiveCoding					= false;
		bBuildDeveloperTools			= false;
		bBuildWithEditorOnlyData		= false;
		bCompileAgainstEngine			= false;
		bCompileAgainstCoreUObject		= true;
		bCompileAgainstApplicationCore	= true;
		bUsesSlate						= true;
		bForceDisableAutomationTests	= true;

		bIsBuildingConsoleApplication = false;

		WindowsPlatform.bUseBundledDbgHelp = false;
		WindowsPlatform.bPixProfilingEnabled = false;
		WindowsPlatform.bMergeIdenticalCOMDATs = true;
		WindowsPlatform.bStripUnreferencedSymbols = true;

		GlobalDefinitions.Add("WITH_AUTOMATION_TESTS=0");
		GlobalDefinitions.Add("WITH_AUTOMATION_WORKER=0");
	}
}
